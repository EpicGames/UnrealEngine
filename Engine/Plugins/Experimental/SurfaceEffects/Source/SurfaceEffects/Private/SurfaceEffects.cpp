// Copyright Epic Games, Inc. All Rights Reserved.

#include "SurfaceEffects.h"

#define LOCTEXT_NAMESPACE "FSurfaceEffectsModule"

void FSurfaceEffectsModule::StartupModule()
{
}

void FSurfaceEffectsModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FSurfaceEffectsModule, SurfaceEffects)