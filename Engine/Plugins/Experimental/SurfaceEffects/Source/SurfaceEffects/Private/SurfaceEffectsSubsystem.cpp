// Copyright Epic Games, Inc. All Rights Reserved.

#include "SurfaceEffectsSubsystem.h"

#include "HAL/IConsoleManager.h"
#include "SurfaceEffectsSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SurfaceEffectsSubsystem)

namespace SurfaceEffectConsoleVariables
{
	int32 bEnabled = 1;
	FAutoConsoleVariableRef CVarEnabled(
	TEXT("SurfaceEffects.Enabled"), bEnabled,
	TEXT("Enables the Surface Effects System.\n")
		TEXT("0: Disabled, 1: Enabled"),
	ECVF_Default);
} // namespace SurfaceEffectConsoleVariables

void USurfaceEffectsSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	const USurfaceEffectsSettings* Settings = GetDefault<USurfaceEffectsSettings>();

	if(Settings)
	{
		SurfaceEffectsData = LoadObject<UDataTable>(nullptr, *Settings->SurfaceEffectsDataTable.ToString(), nullptr, LOAD_None, nullptr);
	}
}

bool USurfaceEffectsSubsystem::IsEnabled()
{
	return SurfaceEffectConsoleVariables::bEnabled != 0;
}
