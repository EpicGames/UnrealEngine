// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/DeveloperSettings.h"

#include "SurfaceEffectsSettings.generated.h"

UCLASS(MinimalAPI, config=Engine, defaultconfig, meta=(DisplayName="Surface Effects"))
class USurfaceEffectsSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	/**
	 * Data table for storing surface effect rules @see FSurfaceEffectTableRow
	 */
	UPROPERTY(config, EditAnywhere, Category = GameplayTags, meta = (AllowedClasses = "/Script/Engine.DataTable", RowType = "/Script/SurfaceEffects.SurfaceEffectTableRow"))
	FSoftObjectPath SurfaceEffectsDataTable;
};
