// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/DataAsset.h"
#include "Engine/DataTable.h"
#include "Subsystems/GameInstanceSubsystem.h"

#include "SurfaceEffectsSubsystem.generated.h"

enum EPhysicalSurface : int;

/**
 * Base context for determining which enum value to return based on a certain rule.
 * We assume most surface interactions will want to use EPhysicalSurface as part of the context.
 */
struct FSurfaceEffectContextBase
{
	explicit FSurfaceEffectContextBase(const EPhysicalSurface PhysicalSurface)
		: PhysicalSurface(PhysicalSurface)
	{
	}
	
	EPhysicalSurface PhysicalSurface;
};

/**
 *	TSurfaceEffectResult - results of a GetSurface call
 */
template <typename TEnum>
struct TSurfaceEffectResult
{
	TEnum OutSurface;
	
	/** Set to true if we successfully got a surface */
	bool bSuccess = false;
};


/**
 * Base data asset used to store what conditions result in a specific surface being returned
 */
UCLASS(MinimalAPI, Abstract)
class USurfaceEffectRule : public UDataAsset
{
	GENERATED_BODY()

public:
	/**
	 * 
	 * @param OutSurfaceValue The enum value to be used when casting back to the enum class
	 * @param Context Context data used by the rule to determine what value to return
	 * @return Returns true if a valid enum value is outputted. Otherwise, return false
	 */
	virtual bool GetSurface(uint8& OutSurfaceValue, const FSurfaceEffectContextBase& Context)
	{
		return false;
	}
};

/**
 * Data Table Row that effectively wraps the Surface Effect Rule
 */
USTRUCT()
struct FSurfaceEffectTableRow : public FTableRowBase
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere , Category="Surface Effect")
	TObjectPtr<USurfaceEffectRule> Rule;
};

/**
 * A system for handling various surface enums based on contexts
 */
UCLASS(MinimalAPI)
class USurfaceEffectsSubsystem : public UGameInstanceSubsystem
{
	GENERATED_BODY()

public:
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;

	template <typename TEnum>
	TSurfaceEffectResult<TEnum> GetSurface(const FSurfaceEffectContextBase& Context)
	{
		TSurfaceEffectResult<TEnum> Result;
		if(IsEnabled() && SurfaceEffectsData)
		{
			static_assert(TIsEnum<TEnum>::Value, "Should only call this with enum types");
			const UEnum* EnumClass = StaticEnum<TEnum>();
			if(EnumClass)
			{
				const FString ContextString(TEXT("USurfaceEffectsSubsystem::GetSurface"));
				const FSurfaceEffectTableRow* Row = SurfaceEffectsData->FindRow<FSurfaceEffectTableRow>(EnumClass->GetFName(), ContextString);
				if(Row && Row->Rule)
				{
					const UEnum* Enum = StaticEnum<TEnum>();
					uint8 OutSurfaceIndex;
					Result.bSuccess = Row->Rule->GetSurface(OutSurfaceIndex, Context);
					Result.bSuccess &= OutSurfaceIndex < Enum->GetMaxEnumValue();
					if(Result.bSuccess)
					{
						Result.OutSurface = static_cast<TEnum>(OutSurfaceIndex);
					}
				}
			}
		}
		return Result;
	}

private:
	/*
	 * We store the UEnum name as the row name in a data table to get the rule associated with that surface enum
	 */
	UPROPERTY()
	TObjectPtr<UDataTable> SurfaceEffectsData;

	SURFACEEFFECTS_API static bool IsEnabled();
};
