// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SurfaceEffects : ModuleRules
{
	public SurfaceEffects(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"DeveloperSettings",
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
			}
			);
	}
}
