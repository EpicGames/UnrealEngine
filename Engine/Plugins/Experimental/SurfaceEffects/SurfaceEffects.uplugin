{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "SurfaceEffects",
	"Description": "A flexible, context-driven surface system",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "SurfaceEffects",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}